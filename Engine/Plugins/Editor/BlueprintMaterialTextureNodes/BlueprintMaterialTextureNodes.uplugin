{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Blueprint Material and Texture Nodes",
	"Description": "Adds blueprint editor-only nodes for reading textures and render targets as well as creating and modifiying Material Instance Constants",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "BlueprintMaterialTextureNodes",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreLoadingScreen"
		}
	],
	"IsExperimentalVersion": false
}