// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class BlueprintMaterialTextureNodes : ModuleRules
{
	public BlueprintMaterialTextureNodes(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
			{
				"Engine",
			});

		PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"ImageCore",
				"RHI",
				"UnrealEd",
			});
	}
}
