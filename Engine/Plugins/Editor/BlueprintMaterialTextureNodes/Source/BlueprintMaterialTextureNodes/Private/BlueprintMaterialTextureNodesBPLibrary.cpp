// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintMaterialTextureNodesBPLibrary.h"
#include "Engine/Texture2D.h"

//RHI gives access to MaxShaderPlatform and FeatureLevel (i.e. GMaxRHIShaderPlatform)

//includes for asset creation
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Materials/MaterialInstanceBasePropertyOverrides.h"
#include "PackageTools.h"
#include "Editor.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Logging/MessageLog.h"

//Material and texture includes
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialShared.h"
#include "ImageCoreUtils.h"
#include "Misc/PackageName.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintMaterialTextureNodesBPLibrary)

#define LOCTEXT_NAMESPACE "BlueprintMaterialTextureLibrary"

UBlueprintMaterialTextureNodesBPLibrary::UBlueprintMaterialTextureNodesBPLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

FLinearColor UBlueprintMaterialTextureNodesBPLibrary::Texture2D_SampleUV_EditorOnly(UTexture2D* Texture, FVector2D UV)
{
#if WITH_EDITOR
	if (Texture != nullptr)
	{
		int NumMips = Texture->GetNumMips();
		int Mip = 0;
		FTexture2DMipMap* CurMip = &Texture->GetPlatformData()->Mips[Mip];
		FByteBulkData* ImageData = &CurMip->BulkData;

		int32 MipWidth = CurMip->SizeX;
		int32 MipHeight = CurMip->SizeY;
		int32 X = FMath::Clamp(int(UV.X * MipWidth), 0, MipWidth - 1);
		int32 Y = FMath::Clamp(int(UV.Y * MipHeight), 0, MipHeight - 1);

		//Retrieve from BulkData if available and if uncompressed format
		if ( (ImageData->IsBulkDataLoaded() && ImageData->GetBulkDataSize() > 0) && (Texture->GetPixelFormat() == PF_B8G8R8A8 || Texture->GetPixelFormat() == PF_FloatRGBA) )
		{
			int32 texelindex = FMath::Min(MipWidth * MipHeight, Y * MipWidth + X);

			if (Texture->GetPixelFormat() == PF_B8G8R8A8)
			{
				FColor* MipData = static_cast<FColor*>(ImageData->Lock(LOCK_READ_ONLY));
				FColor Texel = MipData[texelindex];
				ImageData->Unlock();

				if (Texture->SRGB)
					return Texel;
				else
				{
					return FLinearColor(float(Texel.R), float(Texel.G), float(Texel.B), float(Texel.A)) / 255.0f;
				}
			}
			else if (Texture->GetPixelFormat() == PF_FloatRGBA)
			{
				FFloat16Color* MipData = static_cast<FFloat16Color*>(ImageData->Lock(LOCK_READ_ONLY));
				FFloat16Color Texel = MipData[texelindex];

				ImageData->Unlock();
				return FLinearColor(float(Texel.R), float(Texel.G), float(Texel.B), float(Texel.A));
			}
		}
		//Read Texture Source if platform data is unavailable
		else if ( Texture->Source.IsValid() )
		{
			FTextureSource& TextureSource = Texture->Source;

			// gets a copy of the whole mip to sample one pixel :
			TArray64<uint8> SourceData;
			verify( Texture->Source.GetMipData(SourceData, Mip) );
			ETextureSourceFormat SourceFormat = TextureSource.GetFormat();
			int32 Index = ((Y * MipWidth) + X) * TextureSource.GetBytesPerPixel();
			const uint8* PixelPtr = SourceData.GetData() + Index;
			
			ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(SourceFormat);
			FLinearColor Color = ERawImageFormat::GetOnePixelLinear(PixelPtr,RawFormat,Texture->SRGB);
			return Color;
		}

	}
	FMessageLog("Blueprint").Warning(LOCTEXT("Texture2D_SampleUV_InvalidTexture", "Texture2D_SampleUV_EditorOnly: Texture2D must be non-null."));

#else
	FMessageLog("Blueprint").Error(LOCTEXT("Texture2D_SampleUV_CannotBeSampledAtRuntime", "Texture2D_SampleUV: Can't sample Texture2D at run time."));
#endif

	return FLinearColor(0, 0, 0, 0);
}



TArray<FLinearColor> UBlueprintMaterialTextureNodesBPLibrary::RenderTarget_SampleRectangle_EditorOnly(UTextureRenderTarget2D* InRenderTarget, FLinearColor InRect)
{
#if WITH_EDITOR

	if (!InRenderTarget)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("RenderTargetSampleUV_InvalidRenderTarget", "RenderTargetSampleUVEditoOnly: Render Target must be non-null."));
		return { FLinearColor(0,0,0,0) };
	}
	else if (!InRenderTarget->GetResource())
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("RenderTargetSampleUV_ReleasedRenderTarget", "RenderTargetSampleUVEditoOnly: Render Target has been released."));
		return { FLinearColor(0,0,0,0) };
	}
	else
	{
		ETextureRenderTargetFormat format = (InRenderTarget->RenderTargetFormat);

		if ((format == (RTF_RGBA16f)) || (format == (RTF_RGBA32f)) || (format == (RTF_RGBA8)))
		{

			FTextureRenderTargetResource* RTResource = InRenderTarget->GameThread_GetRenderTargetResource();

			InRect.R = FMath::Clamp(int(InRect.R), 0, InRenderTarget->SizeX - 1);
			InRect.G = FMath::Clamp(int(InRect.G), 0, InRenderTarget->SizeY - 1);
			InRect.B = FMath::Clamp(int(InRect.B), int(InRect.R + 1), InRenderTarget->SizeX);
			InRect.A = FMath::Clamp(int(InRect.A), int(InRect.G + 1), InRenderTarget->SizeY);
			FIntRect Rect = FIntRect(InRect.R, InRect.G, InRect.B, InRect.A);

			FReadSurfaceDataFlags ReadPixelFlags(RCM_MinMax);

			TArray<FColor> OutLDR;
			TArray<FLinearColor> OutHDR;

			TArray<FLinearColor> OutVals;

			bool ishdr = ((format == (RTF_R16f)) || (format == (RTF_RG16f)) || (format == (RTF_RGBA16f)) || (format == (RTF_R32f)) || (format == (RTF_RG32f)) || (format == (RTF_RGBA32f)));

			if (!ishdr)
			{
				RTResource->ReadPixels(OutLDR, ReadPixelFlags, Rect);
				for (auto i : OutLDR)
				{
					OutVals.Add(FLinearColor(float(i.R), float(i.G), float(i.B), float(i.A)) / 255.0f);
				}
			}
			else
			{
				RTResource->ReadLinearColorPixels(OutHDR, ReadPixelFlags, Rect);
				return OutHDR;
			}

			return OutVals;
		}
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("RenderTarget_SampleRectangle_InvalidTexture", "RenderTarget_SampleRectangle_EditorOnly: Currently only 4 channel formats are supported: RTF_RGBA8, RTF_RGBA16f, and RTF_RGBA32f."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("RenderTarget_SampleRectangle_CannotBeSampledAtRuntime", "RenderTarget_SampleRectangle: Can't sample Render Target at run time."));
#endif
	return { FLinearColor(0,0,0,0) };
}

FLinearColor UBlueprintMaterialTextureNodesBPLibrary::RenderTarget_SampleUV_EditorOnly(UTextureRenderTarget2D* InRenderTarget, FVector2D UV)
{
#if WITH_EDITOR

	if (!InRenderTarget)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("RenderTargetSampleUV_InvalidRenderTarget", "RenderTargetSampleUVEditoOnly: Render Target must be non-null."));
		return FLinearColor(0, 0, 0, 0);
	}
	else if (!InRenderTarget->GetResource())
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("RenderTargetSampleUV_ReleasedRenderTarget", "RenderTargetSampleUVEditoOnly: Render Target has been released."));
		return FLinearColor(0, 0, 0, 0);
	}
	else
	{
		ETextureRenderTargetFormat format = (InRenderTarget->RenderTargetFormat);

		if ((format == (RTF_RGBA16f)) || (format == (RTF_RGBA32f)) || (format == (RTF_RGBA8)))
		{
			FTextureRenderTargetResource* RTResource = InRenderTarget->GameThread_GetRenderTargetResource();

			UV.X *= InRenderTarget->SizeX;
			UV.Y *= InRenderTarget->SizeY;
			UV.X = FMath::Clamp(UV.X, 0.0f, float(InRenderTarget->SizeX) - 1);
			UV.Y = FMath::Clamp(UV.Y, 0.0f, float(InRenderTarget->SizeY) - 1);

			FIntRect Rect = FIntRect(UV.X, UV.Y, UV.X + 1, UV.Y + 1);
			FReadSurfaceDataFlags ReadPixelFlags(RCM_MinMax);

			TArray<FColor> OutLDR;
			TArray<FLinearColor> OutHDR;
			TArray<FLinearColor> OutVals;

			bool ishdr = ((format == (RTF_R16f)) || (format == (RTF_RG16f)) || (format == (RTF_RGBA16f)) || (format == (RTF_R32f)) || (format == (RTF_RG32f)) || (format == (RTF_RGBA32f)));

			if (!ishdr)
			{
				RTResource->ReadPixels(OutLDR, ReadPixelFlags, Rect);
				for (auto i : OutLDR)
				{
					OutVals.Add(FLinearColor(float(i.R), float(i.G), float(i.B), float(i.A)) / 255.0f);
				}
			}
			else
			{
				RTResource->ReadLinearColorPixels(OutHDR, ReadPixelFlags, Rect);
				return OutHDR[0];
			}

			return OutVals[0];
		}
		FMessageLog("Blueprint").Warning(LOCTEXT("RenderTarget_SampleUV_InvalidTexture", "RenderTarget_SampleUV_EditorOnly: Currently only 4 channel formats are supported: RTF_RGBA8, RTF_RGBA16f, and RTF_RGBA32f."));
	}
#else
	FMessageLog("Blueprint").Error(LOCTEXT("RenderTarget_SampleUV_CannotBeSampledAtRuntime", "RenderTarget_SampleUV: Can't sample Render Target at run time."));
#endif
	return FLinearColor(0, 0, 0, 0);
}

UMaterialInstanceConstant* UBlueprintMaterialTextureNodesBPLibrary::CreateMIC_EditorOnly(UMaterialInterface* Material, FString InName)
{
#if WITH_EDITOR
	TArray<UObject*> ObjectsToSync;

	if (Material != nullptr)
	{
		// Create an appropriate and unique name 
		FString Name;
		FString PackageName;
		IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		//Use asset name only if directories are specified, otherwise full path
		if (!InName.StartsWith(TEXT("/")))
		{
			FString AssetName = Material->GetOutermost()->GetName();
			const FString SanitizedBasePackageName = UPackageTools::SanitizePackageName(AssetName);
			const FString PackagePath = FPackageName::GetLongPackagePath(SanitizedBasePackageName) + TEXT("/");
			AssetTools.CreateUniqueAssetName(PackagePath, InName, PackageName, Name);
		}
		else
		{
			AssetTools.CreateUniqueAssetName(InName, TEXT(""), PackageName, Name);
		}

		UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
		Factory->InitialParent = Material;

		UObject* NewAsset = AssetTools.CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), UMaterialInstanceConstant::StaticClass(), Factory);

		ObjectsToSync.Add(NewAsset);
		GEditor->SyncBrowserToObjects(ObjectsToSync);

		UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(NewAsset);

		return MIC;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("CreateMIC_InvalidMaterial", "CreateMIC_EditorOnly: Material must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("CreateMIC_CannotBeCreatedAtRuntime", "CreateMIC: Can't create MIC at run time."));
#endif
	return nullptr;
}

//TODO: make this function properly broadcast update to editor to refresh MIC window
void UBlueprintMaterialTextureNodesBPLibrary::UpdateMIC(UMaterialInstanceConstant* MIC)
{
#if WITH_EDITOR
	FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::Default, GMaxRHIShaderPlatform);
	UpdateContext.AddMaterialInstance(MIC);
	MIC->MarkPackageDirty();
#endif
}


bool UBlueprintMaterialTextureNodesBPLibrary::SetMICScalarParam_EditorOnly(UMaterialInstanceConstant* Material, FString ParamName, float Value)
{
#if WITH_EDITOR
	FName Name = FName(*ParamName);

	if (Material != nullptr)
	{
		Material->SetScalarParameterValueEditorOnly(Name, Value);
		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICScalarParam_InvalidMIC", "SetMICScalarParam_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICScalarParam_CannotBeModifiedAtRuntime", "SetMICScalarParam: Can't modify MIC at run time."));
#endif
	return 0;
}

bool UBlueprintMaterialTextureNodesBPLibrary::SetMICVectorParam_EditorOnly(UMaterialInstanceConstant* Material, FString ParamName, FLinearColor Value)
{
#if WITH_EDITOR
	FName Name = FName(*ParamName);

	if (Material != nullptr)
	{
		Material->SetVectorParameterValueEditorOnly(Name, Value);
		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICVectorParam_InvalidMIC", "SetMICVectorParam_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICVectorParam_CannotBeModifiedAtRuntime", "SetMICVectorParam: Can't modify MIC at run time."));
#endif
	return 0;
}

bool UBlueprintMaterialTextureNodesBPLibrary::SetMICTextureParam_EditorOnly(UMaterialInstanceConstant* Material, FString ParamName, UTexture2D* Texture)
{
#if WITH_EDITOR
	FName Name = FName(*ParamName);

	if (Material != nullptr)
	{
		Material->SetTextureParameterValueEditorOnly(Name, Texture);
		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICTextureParam_InvalidMIC", "SetMICTextureParam_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICTextureParam_CannotBeModifiedAtRuntime", "SetMICTextureParam: Can't modify MIC at run time."));
#endif
	return 0;
}

bool UBlueprintMaterialTextureNodesBPLibrary::SetMICShadingModel_EditorOnly(UMaterialInstanceConstant* Material, TEnumAsByte<enum EMaterialShadingModel> ShadingModel)
{
#if WITH_EDITOR
	if (Material != nullptr)
	{
		Material->BasePropertyOverrides.bOverride_ShadingModel = true;
		Material->BasePropertyOverrides.ShadingModel = ShadingModel;

		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICShadingModel_InvalidMIC", "SetMICShadingModel_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICShadingModel_CannotBeModifiedAtRuntime", "SetMICShadingModel: Can't modify MIC at run time."));
#endif
	return 0;
}

bool UBlueprintMaterialTextureNodesBPLibrary::SetMICBlendMode_EditorOnly(UMaterialInstanceConstant* Material, TEnumAsByte<enum EBlendMode> BlendMode)
{
#if WITH_EDITOR
	if (Material != nullptr)
	{
		Material->BasePropertyOverrides.bOverride_BlendMode = true;
		Material->BasePropertyOverrides.BlendMode = BlendMode;
		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICBlendMode_InvalidMIC", "SetMICBlendMode_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICBlendMode_CannotBeModifiedAtRuntime", "SetMICBlendMode: Can't modify MIC at run time."));
#endif
	return 0;
}

bool UBlueprintMaterialTextureNodesBPLibrary::SetMICTwoSided_EditorOnly(UMaterialInstanceConstant* Material, bool TwoSided)
{
#if WITH_EDITOR
	if (Material != nullptr)
	{
		Material->BasePropertyOverrides.bOverride_TwoSided = true;
		Material->BasePropertyOverrides.TwoSided = TwoSided;
		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICTwoSided_InvalidMIC", "SetMICTwoSided_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICTwoSided_CannotBeModifiedAtRuntime", "SetMICTwoSided: Can't modify MIC at run time."));
#endif
	return 0;
}

bool UBlueprintMaterialTextureNodesBPLibrary::SetMICIsThinSurface_EditorOnly(UMaterialInstanceConstant* Material, bool bIsThinSurface)
{
#if WITH_EDITOR
	if (Material != nullptr)
	{
		Material->BasePropertyOverrides.bOverride_bIsThinSurface = true;
		Material->BasePropertyOverrides.bIsThinSurface = bIsThinSurface;
		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICIsThinSurface_InvalidMIC", "SetMICIsThinSurface_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICIsThinSurface_CannotBeModifiedAtRuntime", "SetMICIsThinSurface: Can't modify MIC at run time."));
#endif
	return 0;
}

bool UBlueprintMaterialTextureNodesBPLibrary::SetMICDitheredLODTransition_EditorOnly(UMaterialInstanceConstant* Material, bool DitheredLODTransition)
{
#if WITH_EDITOR
	if (Material != nullptr)
	{
		Material->BasePropertyOverrides.bOverride_DitheredLODTransition = true;
		Material->BasePropertyOverrides.DitheredLODTransition = DitheredLODTransition;
		UpdateMIC(Material);

		return 1;
	}
	FMessageLog("Blueprint").Warning(LOCTEXT("SetMICDitheredLODTransition_InvalidMIC", "SetMICDitheredLODTransition_EditorOnly: MIC must be non-null."));
#else
	FMessageLog("Blueprint").Error(LOCTEXT("SetMICDitheredLODTransition_CannotBeModifiedAtRuntime", "SetMICDitherTransition: Can't modify MIC at run time."));
#endif
	return 0;
}

#undef LOCTEXT_NAMESPACE

