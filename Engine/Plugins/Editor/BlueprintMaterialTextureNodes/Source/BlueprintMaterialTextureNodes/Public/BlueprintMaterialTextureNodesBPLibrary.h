// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/EngineTypes.h"
#include "BlueprintMaterialTextureNodesBPLibrary.generated.h"

#define UE_API BLUEPRINTMATERIALTEXTURENODES_API

class UMaterialInstanceConstant;
class UMaterialInterface;
class UTexture2D;
class UTextureRenderTarget2D;

/* 
*	Function library class.
*	Each function in it is expected to be static and represents blueprint node that can be called in any blueprint.
*
*	When declaring function you can define metadata for the node. Key function specifiers will be BlueprintPure and BlueprintCallable.
*	BlueprintPure - means the function does not affect the owning object in any way and thus creates a node without Exec pins.
*	BlueprintCallable - makes a function which can be executed in Blueprints - Thus it has Exec pins.
*	DisplayName - full name of the node, shown when you mouse over the node and in the blueprint drop down menu.
*				Its lets you name the node using characters not allowed in C++ function names.
*	CompactNodeTitle - the word(s) that appear on the node.
*	Keywords -	the list of keywords that helps you to find node when you search for it using Blueprint drop-down menu. 
*				Good example is "Print String" node which you can find also by using keyword "log".
*	Category -	the category your node will be under in the Blueprint drop-down menu.
*
*	For more info on custom blueprint nodes visit documentation:
*	https://docs.unrealengine.com/ProgrammingAndScripting/Blueprints/TechnicalGuide/ExtendingBlueprints
*/
UCLASS(MinimalAPI)
class UBlueprintMaterialTextureNodesBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()


	/**
	* Samples a texel from a Texture 2D with VectorDisplacement Compression
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Texture2D Sample UV Editor Only", Keywords = "Read Texture UV"), Category = Rendering)
	static UE_API FLinearColor Texture2D_SampleUV_EditorOnly(UTexture2D* Texture, FVector2D UV);

	/**
	* Samples an array of values from a Texture Render Target 2D. Currently only 4 channel formats are supported.
	* Only works in the editor
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Render Target Sample Rectangle Editor Only", Keywords = "Sample Render Target Rectangle"), Category = Rendering)
	static UE_API TArray<FLinearColor> RenderTarget_SampleRectangle_EditorOnly(UTextureRenderTarget2D* InRenderTarget, FLinearColor InRect);

	/**
	* Samples a value from a Texture Render Target 2D. Currently only 4 channel formats are supported.
	* Only works in the editor
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Render Target Sample UV Editor Only", Keywords = "Sample Render Target UV"), Category = Rendering)
	static UE_API FLinearColor RenderTarget_SampleUV_EditorOnly(UTextureRenderTarget2D* InRenderTarget, FVector2D UV);


	/**
	* Creates a new Material Instance Constant asset
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Create MIC Editor Only", Keywords = "Create MIC", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API UMaterialInstanceConstant* CreateMIC_EditorOnly(UMaterialInterface* Material, FString Name = "MIC_");

	UFUNCTION()
	static UE_API void UpdateMIC(UMaterialInstanceConstant* MIC);

	/**
	* Sets a Scalar Parameter value in a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC Scalar Parameter Editor Only", Keywords = "Set MIC Scalar Parameter", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICScalarParam_EditorOnly(UMaterialInstanceConstant* Material, FString ParamName = "test", float Value = 0.0f);

	/**
	* Sets a Vector Parameter value in a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC Vector Parameter Editor Only", Keywords = "Set MIC Vector Parameter", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICVectorParam_EditorOnly(UMaterialInstanceConstant* Material, FString ParamName, FLinearColor Value = FLinearColor(0, 0, 0, 0));

	/**
	* Sets a Texture Parameter value in a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC Texture Parameter Editor Only", Keywords = "Set MIC Texture Parameter", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICTextureParam_EditorOnly(UMaterialInstanceConstant* Material, FString ParamName, UTexture2D* Texture = nullptr);

	/**
	* Overrides the Shading Model of a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC Shading Model Editor Only", Keywords = "Set MIC Shading Model", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICShadingModel_EditorOnly(UMaterialInstanceConstant* Material, TEnumAsByte<EMaterialShadingModel> ShadingModel = MSM_DefaultLit);

	/**
	* Overrides the Blend Mode of a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC Blend Mode Editor Only", Keywords = "Set MIC Blend Mode", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICBlendMode_EditorOnly(UMaterialInstanceConstant* Material, TEnumAsByte<EBlendMode> BlendMode = BLEND_Opaque);

	/**
	* Overrides the Two Sided setting of a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC Two Sided Editor Only", Keywords = "Set MIC Two Sided", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICTwoSided_EditorOnly(UMaterialInstanceConstant* Material, bool TwoSided = false);

	/**
	* Overrides the IsThinSurface setting of a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC IsThinSurface Editor Only", Keywords = "Set MIC IsThinSurface", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICIsThinSurface_EditorOnly(UMaterialInstanceConstant* Material, bool bIsThinSurface = false);	

	/**
	* Overrides the Blend Mode of a Material Instance Constant
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set MIC Dithered LOD Editor Only", Keywords = "Set MIC Dithered LOD Transition", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	static UE_API bool SetMICDitheredLODTransition_EditorOnly(UMaterialInstanceConstant* Material, bool DitheredLODTransition = false);
};

#undef UE_API
